{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": [
		"WinGDK"
	],
	"Modules": [
		{
			"Name": "GDKPlatformDLC",
			"Type": "Runtime",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}