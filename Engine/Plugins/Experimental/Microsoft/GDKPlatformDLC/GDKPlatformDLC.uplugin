{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "DLC Support for GDK",
	"Description": "Experimental support for DLC in GDK.",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"ExplicitlyLoaded": true,
	"HasExplicitPlatforms": true,	
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "GDKPlatformDLC",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "PlatformDLC",
			"Enabled": true
		}
	]
}
