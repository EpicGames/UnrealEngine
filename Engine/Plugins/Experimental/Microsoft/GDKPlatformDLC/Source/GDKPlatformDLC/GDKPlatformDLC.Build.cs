// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;

public class GDKPlatformDLC : ModuleRules
{
	public GDKPlatformDLC(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddAll( 
			"GRDK",
			"GDKRuntime",
            "GDKPackageManifest",	
			"PlatformDLC",
			"Core",
			"CoreUObject",
			"Engine",
			"Projects",
			"AssetRegistry"
		);

		PublicIncludePathModuleNames.Add("GDKRuntime");
	}
}
