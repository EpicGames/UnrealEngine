// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKPlatformDLC.h"
#if WITH_GRDK
#include "GDKTaskQueueHelpers.h"
#include "GDKThreadCheck.h"
#include "GDKRuntimeModule.h"
#include "IGDKPackageManifestModule.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/CString.h"
#include "Async/Async.h"
#include "Interfaces/IPluginManager.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "PlatformDLCModule.h"

THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END

// when this is 1 the code will show a normal message box instead of opening the real purchase UI, for debugging purposes
#if !defined(GDK_DLC_MOCK_PURCHASE_UI)
#define GDK_DLC_MOCK_PURCHASE_UI 0
#endif


FGDKPlatformDLC::FGDKPlatformDLC()
	: CurrentStoreContext(nullptr)
{
	UserLoginChangedDelegateHandle = FCoreDelegates::OnUserLoginChangedEvent.AddRaw(this, &FGDKPlatformDLC::OnUserLoginChanged);
}


FGDKPlatformDLC::~FGDKPlatformDLC()
{
	if (IsInitialized())
	{
		Shutdown();
	}
	FCoreDelegates::OnUserLoginChangedEvent.Remove(UserLoginChangedDelegateHandle);
}

void FGDKPlatformDLC::BeginInitializeInternal()
{
	check(ProcessingState == EProcessingState::Idle);
	ProcessingState = EProcessingState::Initializing;

	InitFuture = Async(EAsyncExecution::Thread, [this]()
	{
		if (ProcessingState == EProcessingState::Destroying)
		{
			return;
		}

		// read configuration
		GConfig->GetBool(ConfigSectionName, TEXT("AutoMountDLC"), bAutoMountDLC, GEngineIni);
		GConfig->GetBool(ConfigSectionName, TEXT("AutoMountExistingDLC"), bAutoMountExistingDLC, GEngineIni);
		GConfig->GetBool(ConfigSectionName, TEXT("AutoMountPakFiles"), bAutoMountPakFiles, GEngineIni);
		GConfig->GetBool(ConfigSectionName, TEXT("AutoMountPlugin"), bAutoMountPlugin, GEngineIni);
		GConfig->GetBool(ConfigSectionName, TEXT("AutoMountAgeRestricted"), bAutoMountAgeRestricted, GEngineIni);

		// make sure the package manifest module is loaded
		IGDKPackageManifestModule::Get();

		// select the default store user
		TArray<FGDKUserHandle> AllUsers = IGDKRuntimeModule::Get().GetAllUserHandles();
		if (AllUsers.Num() > 0)
		{
			SetStoreUserHandle(AllUsers[0]);
		}

		// build the StoreId map
		TArray<FString> DLCPackages;
		if (GConfig->GetArray(IGDKRuntimeModule::Get().GetConfigSectionName(), TEXT("DLCPackages"), DLCPackages, GEngineIni))
		{
			FScopeLock Lock(&PackageDataLock);
			for (FString& DLCPackage : DLCPackages)
			{
				DLCPackage.TrimStartAndEndInline();
				DLCPackage.ReplaceInline(TEXT("("), TEXT(""));
				DLCPackage.ReplaceInline(TEXT(")"), TEXT(""));

				FString DLCName;
				FString StoreId;
				if (FParse::Value(*DLCPackage, TEXT("DLCName="), DLCName) && FParse::Value(*DLCPackage, TEXT("StoreId="), StoreId))
				{
					DLCNameToStoreIdMap.Add(FName(*DLCName), StoreId);

					TSharedPtr<FPackageData> Package = AllPackages.Add_GetRef(MakeShared<FPackageData>(this, FName(*DLCName), StoreId));
					Package->Context.Package = Package;

					AllStoreIds.Add(Package->StoreId);
				}
			}
		}

		// build the plugin map
		TArray<FString> DLCPlugins;
		if (GConfig->GetArray(IGDKRuntimeModule::Get().GetConfigSectionName(), TEXT("DLCPlugins"), DLCPlugins, GEngineIni))
		{
			FScopeLock Lock(&PackageDataLock);
			for (FString& DLCPlugin : DLCPlugins)
			{
				DLCPlugin.TrimStartAndEndInline();
				DLCPlugin.ReplaceInline(TEXT("("), TEXT(""));
				DLCPlugin.ReplaceInline(TEXT(")"), TEXT(""));

				FString DLCName;
				FString Plugin;
				if (FParse::Value(*DLCPlugin, TEXT("DLCName="), DLCName) && FParse::Value(*DLCPlugin, TEXT("Plugin="), Plugin))
				{
					if (ensure(DLCNameToStoreIdMap.Contains(FName(*DLCName))))
					{
						DLCNameToPluginNameMap.Add(FName(*DLCName), Plugin);
					}
					else
					{
						UE_LOGF(LogPlatformDLC, Warning, "Ignoring DLCPlugin %ls because DLC %ls is not known", *Plugin, *DLCName);
					}
				}
			}
		}

		// create package data for any already-installed DLC
		auto EnumeratePackageCallback = [](void* Context, const XPackageDetails* Details)
		{
			FGDKPlatformDLC* This = (FGDKPlatformDLC*)Context;
			This->AsyncInitDLC(*Details, This->bAutoMountExistingDLC, false);
			return true;
		};
		XPackageEnumeratePackages( XPackageKind::Content, XPackageEnumerationScope::ThisOnly, this, EnumeratePackageCallback );

		// register a callback when new packages are downloaded
		auto PackageInstalledCallback = [](void* Context, const XPackageDetails* Details)
		{
			FGDKPlatformDLC* This = (FGDKPlatformDLC*)Context;
			This->AsyncInitDLC(*Details, This->bAutoMountDLC, true);
		};
		XPackageRegisterPackageInstalled(FGDKAsyncTaskQueue::GetGenericQueue(), this, PackageInstalledCallback, &PackageInstalledToken);

		// start out by querying the store to find out which DLC we already own
		if (AllPackages.Num() > 0)
		{
			bPendingProductsRequery = true;
			AddToTicker();
		}

		ProcessingState = EProcessingState::Idle;
		EndInitializeInternal();
	});
}

void FGDKPlatformDLC::Shutdown()
{
	// try to stop any current processing tasks
	ProcessingState = EProcessingState::Destroying;
	if (InitFuture.IsValid())
	{
		InitFuture.Wait();
	}
	CurrentProcessingAsyncTask.TryCancel(true);
	for (TSharedPtr<FPackageData>& Package : AllPackages)
	{
		Package->DownloadAsyncTask.TryCancel(true);
	}

	XPackageUnregisterPackageInstalled(PackageInstalledToken, true);

	{
		FScopeLock Lock(&PackageDataLock);
		PendingPackages.Reset();
		LostLicensePackages.Reset();
		CurrentProcessingPackage.Reset();
	}

	// release all packages (only the package captured by AsyncGDKTask blocks will be kept alive)
	AllPackages.Reset();

	if (CurrentStoreContext != nullptr)
	{
		XStoreCloseContextHandle(CurrentStoreContext);
		CurrentStoreContext = nullptr;
	}

	RemoveFromTicker();

	FGenericPlatformDLC::Shutdown();
}



bool FGDKPlatformDLC::Tick( float DeltaSeconds )
{
	switch (ProcessingState)
	{
		case EProcessingState::Initializing:
		{
		}
		break;

		case EProcessingState::Idle:
		{
			FScopeLock Lock(&PackageDataLock);

			check(!CurrentProcessingPackage.IsValid());

			if (bPendingProductsRequery)
			{
				bPendingProductsRequery = false;
				ProcessingState = EProcessingState::QueryingProducts;
				AsyncQueryingProducts_GameThread();
				break;
			}
			else if (PendingPackages.Num() > 0)
			{
				// get the next package to mount
				CurrentProcessingPackage = PendingPackages.Pop();
				BeginProcessingPackage_GameThread(CurrentProcessingPackage);
			}
			else if (LostLicensePackages.Num() > 0)
			{
				// get the next lost license package
				CurrentProcessingPackage = LostLicensePackages.Pop();
				LicenseLost_GameThread(CurrentProcessingPackage);
				CurrentProcessingPackage.Reset();
			}
			else
			{
				// there's no more work to do
				RemoveFromTicker();
				return false;
			}
		}
		break;

		case EProcessingState::QueryingProducts:
		{
			// nothing to do while this is in progress
		}
		break;

		case EProcessingState::Purchase_QueryingProduct:
		case EProcessingState::Purchase_ShowingPurchaseUI:
		case EProcessingState::Mount_AcquiringLicense:
		case EProcessingState::Mount_MountingPackage:
		case EProcessingState::Mount_CachingData:
		case EProcessingState::Mount_MountingPlugin:
		{
			// nothing to do, but sanity check that the package is still valid
			check(CurrentProcessingPackage.IsValid());
		}
		break;

		case EProcessingState::Mount_Completed:
		{
			Mount_Completed_GameThread(CurrentProcessingPackage);
			CurrentProcessingPackage.Reset();
		}
		break;

		case EProcessingState::Purchase_Completed:
		{
			Purchase_Completed_GameThread(CurrentProcessingPackage);
			CurrentProcessingPackage.Reset();
		}
		break;

		case EProcessingState::Purchase_DownloadRequested:
		{
			ProcessingState = EProcessingState::Idle;
			CurrentProcessingPackage.Reset();
		}
		break;
	}

	// keep ticking
	return true;
}


void FGDKPlatformDLC::AsyncInitDLC(const XPackageDetails& Details, bool bMount, bool bNotify)
{
	UE_CLOG(Details.installing, LogPlatformDLC, Error, TEXT("package %hs is still installing - intelligent delivery is not supported for DLC at the moment"), Details.packageIdentifier );

	TSharedPtr<FPackageData> Package = FindDLCByStoreId( Details.storeId );
	if (!Package.IsValid())
	{
		UE_LOGF(LogPlatformDLC, Warning, "Ignoring DLC with StoreId %ls because it isn't referenced in DLCPackages config array", UTF8_TO_TCHAR(Details.storeId));
		return;
	}

	UE_LOG(LogPlatformDLC, Log, TEXT("Downloaded package %hs for DLC %s"), Details.packageIdentifier, *Package->DLCName.ToString() );
	{
		FScopeLock Lock(&PackageDataLock);
		Package->OnDownloaded(Details);
	}

	if (bNotify)
	{
		PostNotification(Package->DLCName, ENotification::Downloaded, true);
	}

	if (bMount)
	{
		AddToPendingPackages( Package );
	}
}


bool FGDKPlatformDLC::AddToPendingPackages( TSharedPtr<FPackageData> Package )
{
	FScopeLock Lock(&PackageDataLock);
	PendingPackages.AddUnique(Package);
	AddToTicker();

	return true;
}


void FGDKPlatformDLC::AsyncQueryingProducts_GameThread()
{
	check(ProcessingState == EProcessingState::QueryingProducts);

	if (!EnsureValidStoreContext())
	{
		// No valid store user - clear all purchase states
		FScopeLock Lock(&PackageDataLock);
		for (TSharedPtr<FPackageData>& Package : AllPackages)
		{
			Package->bIsPurchased = false;
		}

		ProcessingState = EProcessingState::Idle;
		return;
	}

	HRESULT hResult = AsyncGDKTask(
		CurrentProcessingAsyncTask,
		[this](XAsyncBlock* Block)
		{
			const char** StoreIds = AllStoreIds.GetData();
			return XStoreQueryProductsAsync(CurrentStoreContext, XStoreProductKind::Durable, StoreIds, AllStoreIds.Num(), nullptr, 0, Block);
		},
		[this, QueryUser = CurrentStoreUser](XAsyncBlock* Block)
		{
			// early out if the DLC system is being destroyed
			bool bCancel = (ProcessingState == EProcessingState::Destroying);
			if (bCancel)
			{
				return;
			}

			// The store user changed while query was in flight so the results are invalid
			if (QueryUser != CurrentStoreUser)
			{
				ProcessingState = EProcessingState::Idle;
				return;
			}

			ProcessQueryProductsResult(Block);
			ProcessingState = EProcessingState::Idle;
		}
	);

	if (FAILED(hResult))
	{
		ProcessingState = EProcessingState::Idle;
		UE_LOG(LogPlatformDLC, Error, TEXT("Failed to start querying DLC products: 0x%X"), hResult);
	}
}

void FGDKPlatformDLC::BeginProcessingPackage_GameThread( TSharedPtr<FPackageData> Package )
{
	// if the package is already mounted and ready, just fire the callbacks again
	bool bHasLicense = (Package->LicenseState == ELicenseState::Licensed);
	if (Package->MountState == EMountState::Mounted && bHasLicense)
	{
		ProcessingState = EProcessingState::Mount_Completed;
		return;
	}

	// check for valid state
	if (bHasLicense && Package->MountState != EMountState::NotMounted && Package->MountState != EMountState::Unmounted)
	{
		checkNoEntry();
		return;
	}

	// early out if we are ignoring age-restricted DLC
	if (Package->bIsAgeRestricted && !bAutoMountAgeRestricted)
	{
		Package->MountState = EMountState::Ignored_AgeRestricted;
		ProcessingState = EProcessingState::Mount_Completed;
		UE_LOGF(LogPlatformDLC, Log, "Ignoring DLC %ls because it is age restricted for one or more of the currently logged in users", *Package->PackageName );
		return;
	}

	// purchase before downloading
	if (Package->DownloadState == EDownloadState::NotDownloaded && !Package->bIsPurchased && Package->bAllowPurchasePrompt)
	{
		ProcessingState = EProcessingState::Purchase_ShowingPurchaseUI;
		Purchase_AsyncShowPurchaseUI_GameThread(Package);
		return;
	}

	// start the mounting process by getting a license
	ProcessingState = EProcessingState::Mount_AcquiringLicense;
	Mount_AsyncAcquireLicense_GameThread(Package);
}



void FGDKPlatformDLC::Purchase_AsyncShowPurchaseUI_GameThread( TSharedPtr<FPackageData> Package )
{
	check(ProcessingState == EProcessingState::Purchase_ShowingPurchaseUI);

	if (!EnsureValidStoreContext())
	{
		Package->LicenseState = ELicenseState::Unlicensed;
		ProcessingState = EProcessingState::Purchase_Completed;
		return;
	}

	HRESULT hResult = AsyncGDKTask(
		CurrentProcessingAsyncTask,
		[Package, this](XAsyncBlock* Block)
		{
#if GDK_DLC_MOCK_PURCHASE_UI && !UE_BUILD_SHIPPING
			return XGameUiShowMessageDialogAsync(Block, "Mock purchase dialog", Package->StoreId, "OK", "Cancel", nullptr, XGameUiMessageDialogButton::First, XGameUiMessageDialogButton::Second);
#else
			return XStoreShowPurchaseUIAsync(CurrentStoreContext, Package->StoreId, nullptr, nullptr, Block);
#endif
		},
		[Package, this](XAsyncBlock* Block)
		{
			// early out if the DLC system is being destroyed
			bool bCancel = (ProcessingState == EProcessingState::Destroying);
			if (bCancel)
			{
				return;
			}	
		
#if GDK_DLC_MOCK_PURCHASE_UI && !UE_BUILD_SHIPPING
			XGameUiMessageDialogButton MockButton;
			HRESULT hResult = XGameUiShowMessageDialogResult(Block, &MockButton);
			if (SUCCEEDED(hResult) && MockButton != XGameUiMessageDialogButton::First)
			{
				hResult = E_ABORT;
			}
#else
			HRESULT hResult = XStoreShowPurchaseUIResult(Block);
#endif
			if (SUCCEEDED(hResult))
			{
				ProcessingState = EProcessingState::Purchase_QueryingProduct;
				Purchase_AsyncQueryingProduct_GameThread(Package);
			}
			else
			{
				Package->LicenseState = ELicenseState::Unlicensed;
				ProcessingState = EProcessingState::Purchase_Completed;
				UE_LOG(LogPlatformDLC, Error, TEXT("Failed to purchase DLC %s : 0x%X"), *Package->DLCName.ToString(), hResult);
			}
		}
	);


	if (FAILED(hResult))
	{
		Package->LicenseState = ELicenseState::Unlicensed;
		ProcessingState = EProcessingState::Purchase_Completed;
		UE_LOG(LogPlatformDLC, Error, TEXT("Failed to open store UI page for DLC %s : 0x%X"), *Package->DLCName.ToString(), hResult);
	}
}

void FGDKPlatformDLC::Purchase_AsyncQueryingProduct_GameThread( TSharedPtr<FPackageData> Package )
{
	check(ProcessingState == EProcessingState::Purchase_QueryingProduct);

	if (!EnsureValidStoreContext())
	{
		ProcessingState = EProcessingState::Purchase_Completed;
		return;
	}

	HRESULT hResult = AsyncGDKTask(
		CurrentProcessingAsyncTask,
		[this, Package](XAsyncBlock* Block)
		{
			const char* StoreIds[] = { Package->StoreId };
			return XStoreQueryProductsAsync(CurrentStoreContext, XStoreProductKind::Durable, StoreIds, 1, nullptr, 0, Block);
		},
		[this, Package](XAsyncBlock* Block)
		{
			// early out if the DLC system is being destroyed
			if (ProcessingState == EProcessingState::Destroying)
			{
				return;
			}

			if (!ProcessQueryProductsResult(Block))
			{
				ProcessingState = EProcessingState::Purchase_Completed;
				return;
			}

			if (!Package->bIsPurchased)
			{
				PostNotification(Package->DLCName, ENotification::Entitlement, false);
				UE_LOG(LogPlatformDLC, Error, TEXT("DLC %s not in user collection after purchase"), *Package->DLCName.ToString());
				ProcessingState = EProcessingState::Purchase_Completed;
				return;
			}

			PostNotification(Package->DLCName, ENotification::Entitlement, true);
			if (Package->DownloadState == EDownloadState::Downloaded)
			{
				ProcessingState = EProcessingState::Mount_AcquiringLicense;
				Mount_AsyncAcquireLicense_GameThread(Package);
			}
			else if (AsyncDownloadPackage(Package))
			{
				ProcessingState = EProcessingState::Purchase_DownloadRequested;
			}
			else
			{
				ProcessingState = EProcessingState::Purchase_Completed;
			}
		}
	);

	if (FAILED(hResult))
	{
		ProcessingState = EProcessingState::Purchase_Completed;
		UE_LOG(LogPlatformDLC, Error, TEXT("Failed to start querying DLC product for %s: 0x%X"), *Package->DLCName.ToString(), hResult);
	}
}

void FGDKPlatformDLC::Purchase_Completed_GameThread( TSharedPtr<FPackageData> Package )
{
	if (Package->DownloadState == EDownloadState::NotDownloaded) // this means we came here via Download()
	{
		PostNotification(Package->DLCName, ENotification::Downloaded, false);
	}
	else // ... otherwise we must have come here via Mount()
	{
		PostNotification(Package->DLCName, ENotification::Mounted, false);
	}

	ProcessingState = EProcessingState::Idle;
}



void FGDKPlatformDLC::Mount_AsyncAcquireLicense_GameThread( TSharedPtr<FPackageData> Package )
{
	check(ProcessingState == EProcessingState::Mount_AcquiringLicense);

	// if we already have a license, check whether it's still valid
	if (Package->DLCLicense != nullptr)
	{
		if (XStoreIsLicenseValid(Package->DLCLicense))
		{
			// license is still valid - skip straight to mounting
			ProcessingState = EProcessingState::Mount_MountingPackage;
			Mount_AsyncMountPackage_GameThread(Package);
			return;
		}

		// license handle is stale - unregister the lost callback, close it, and re-acquire below
		XStoreUnregisterPackageLicenseLost(Package->DLCLicense, Package->PackageLicenseLostToken, true);
		XStoreCloseLicenseHandle(Package->DLCLicense);
		Package->DLCLicense = nullptr;
		Package->LicenseState = ELicenseState::Unlicensed;
		PostNotification(Package->DLCName, ENotification::Entitlement, false);
	}

	// make sure there is a store context
	if (!EnsureValidStoreContext())
	{
		ProcessingState = EProcessingState::Mount_Completed;
		UE_LOGF(LogPlatformDLC, Error, "cannot mount DLC");
		return;
	}

	check(Package->DLCLicense == nullptr);

	HRESULT hResult = AsyncGDKTask(
		CurrentProcessingAsyncTask,
		[Package, this](XAsyncBlock* Block)
		{
			return XStoreAcquireLicenseForPackageAsync(CurrentStoreContext, Package->PackageIdentifier, Block);
		},
		[Package, this](XAsyncBlock* Block)
		{
			HRESULT hResult = XStoreAcquireLicenseForPackageResult(Block, &Package->DLCLicense);

			// early out if the DLC system is being destroyed (must be done after caching the DLC license so we don't leak it)
			bool bCancel = (ProcessingState == EProcessingState::Destroying);
			if (bCancel)
			{
				return;
			}

			if (SUCCEEDED(hResult) && XStoreIsLicenseValid(Package->DLCLicense) )
			{
				Package->LicenseState = ELicenseState::Licensed;

				// register for a callback when the license is lost
				hResult = XStoreRegisterPackageLicenseLost(Package->DLCLicense, FGDKAsyncTaskQueue::GetGenericQueue(), &Package->Context, &FGDKPlatformDLC::OnPackageLicenseLost, &Package->PackageLicenseLostToken );
				UE_CLOGF(FAILED(hResult), LogPlatformDLC, Warning, "Could not register for GDK license lost callback : 0x%X", hResult);

				// start mounting the DLC package
				ProcessingState = EProcessingState::Mount_MountingPackage;
				Mount_AsyncMountPackage_GameThread(Package);
				return;
			}

			// if we received a license that was not valid, close it now as there's nothing we can do with it
			if (Package->DLCLicense != nullptr)
			{
				XStoreUnregisterPackageLicenseLost(Package->DLCLicense, Package->PackageLicenseLostToken, true);
				XStoreCloseLicenseHandle(Package->DLCLicense);
				Package->DLCLicense = nullptr;
			}

			if (Package->bAllowPurchasePrompt)
			{
				ProcessingState = EProcessingState::Purchase_ShowingPurchaseUI;
				Purchase_AsyncShowPurchaseUI_GameThread(Package);
				UE_LOGF(LogPlatformDLC, Log, "Failed to get valid license for package %ls : 0x%X ... starting purchase", *Package->PackageName, hResult);
			}
			else
			{
				Package->LicenseState = ELicenseState::Unlicensed;
				ProcessingState = EProcessingState::Mount_Completed;
				UE_LOGF(LogPlatformDLC, Error, "Failed to get valid license for package %ls : 0x%X", *Package->PackageName, hResult);
			}
		}
	);

	if (FAILED(hResult))
	{
		Package->LicenseState = ELicenseState::Unlicensed;
		ProcessingState = EProcessingState::Mount_Completed;
		UE_LOGF(LogPlatformDLC, Error, "Failed to start getting license for package %ls : 0x%X", *Package->PackageName, hResult);
	}
}


void FGDKPlatformDLC::Mount_AsyncMountPackage_GameThread( TSharedPtr<FPackageData> Package )
{
	check(ProcessingState == EProcessingState::Mount_MountingPackage);

	// if the package is already mounted, it means we've only reaquired the license
	if (Package->DLCHandle != nullptr && ensure(Package->MountState == EMountState::Mounted) )
	{
		ProcessingState = EProcessingState::Mount_Completed;
		return;
	}


	HRESULT hResult = AsyncGDKTask(
		CurrentProcessingAsyncTask,
		[Package](XAsyncBlock* Block)
		{
			return XPackageMountWithUiAsync(Package->PackageIdentifier, Block);
		},
		[Package, this](XAsyncBlock* Block)
		{
			HRESULT hResult = XPackageMountWithUiResult(Block, &Package->DLCHandle);

			// early out if the DLC system is being destroyed (must be done after caching the DLC handle so we don't leak it)
			bool bCancel = (ProcessingState == EProcessingState::Destroying);
			if (bCancel)
			{
				return;
			}

			if (SUCCEEDED(hResult))
			{
				// continue to initialize the DLC package on a background thread
				ProcessingState = EProcessingState::Mount_CachingData;
				AsyncTask( ENamedThreads::AnyBackgroundThreadNormalTask, [Package, this]()
				{
					Mount_CachePackageData_BackgroundThread(Package);
				});
				return;
			}

			Package->MountState = EMountState::NotMounted;
			ProcessingState = EProcessingState::Mount_Completed;
			UE_LOGF(LogPlatformDLC, Error, "Failed to mount DLC '%ls' : 0x%X", *Package->PackageName, hResult);
		}
	);

	if (FAILED(hResult))
	{
		Package->MountState = EMountState::NotMounted;
		ProcessingState = EProcessingState::Mount_Completed;
		UE_LOGF(LogPlatformDLC, Error, "Failed to start mounting mount DLC '%ls' : 0x%X", *Package->PackageName, hResult);
	}
}


void FGDKPlatformDLC::Mount_CachePackageData_BackgroundThread( TSharedPtr<FPackageData> Package )
{
	check(ProcessingState == EProcessingState::Mount_CachingData);

	HRESULT hResult;

	// get the mount path
	char MountPath[MAX_PATH+1];
	hResult = XPackageGetMountPath(Package->DLCHandle, UE_ARRAY_COUNT(MountPath)-1, MountPath);
	if (FAILED(hResult))
	{
		Package->MountState = EMountState::NotMounted;
		ProcessingState = EProcessingState::Mount_Completed;
		UE_LOGF(LogPlatformDLC, Error, "Failed to get mount path for DLC '%ls' : 0x%X", *Package->PackageName, hResult);

		XPackageCloseMountHandle(Package->DLCHandle);
		Package->DLCHandle = nullptr;
		return;
	}
	Package->MountPath = UTF8_TO_TCHAR(MountPath);
	if (!Package->MountPath.EndsWith(TEXT("\\")))
	{
		Package->MountPath += TEXT("\\");
	}

	// load package manifest
	IGDKPackageManifestModule::Get().LoadManifestFromDLC(Package->DLCHandle, Package->PackageIdentifier, Package->MountPath);

	// finalize on the GameThread
	ProcessingState = EProcessingState::Mount_MountingPlugin;
	ExecuteOnGameThread( UE_SOURCE_LOCATION, [this, Package]()
	{
		Mount_MountPlugin_GameThread(Package);
	});
}


void FGDKPlatformDLC::Mount_MountPlugin_GameThread( TSharedPtr<FPackageData> Package )
{
	check(ProcessingState == EProcessingState::Mount_MountingPlugin);

	if (bAutoMountPakFiles)
	{
		// mount all available pak files. NOTE: intelligent delivery is not supported for DLC at the moment - only what is currently installed will be mounted
		TArray<FString> PakFileNames;
		IPlatformFile::GetPlatformPhysical().FindFilesRecursively(PakFileNames, *Package->MountPath, TEXT(".pak") );
		for (const FString& PakFileName : PakFileNames)
		{
			if (IGDKPackageManifestModule::Get().IsPakFileInstalled(Package->DLCHandle, PakFileName))
			{
				IPakFile* PakFile = FCoreDelegates::MountPak.IsBound() ? FCoreDelegates::MountPak.Execute(PakFileName, INDEX_NONE) : nullptr;
				if (PakFile)
				{
					UE_LOGF(LogPlatformDLC, Log, "Successfully mounted %ls", *PakFileName);
				}
				else
				{
					UE_LOGF(LogPlatformDLC, Log, "Failed to mount %ls", *PakFileName);
				}
			}
		}
	}

	if (bAutoMountPlugin && bAutoMountPakFiles && DLCNameToPluginNameMap.Contains(Package->DLCName))
	{
		const FString& PluginName = DLCNameToPluginNameMap.FindChecked(Package->DLCName);
	
		// find & load the .uplugin (should only be one per DLC, as specified by "RunUAT ... -DLCName=<plugin>")
		TArray<FString> FoundPlugins;
		IPlatformFile::GetPlatformPhysical().FindFilesRecursively(FoundPlugins, *Package->MountPath, *FString::Printf(TEXT("%s.uplugin"), *PluginName) );
		checkf(FoundPlugins.Num() != 0, TEXT("found %d plugins in %s - DLC should have 1!"), FoundPlugins.Num(), *Package->MountPath);

		Package->DLCPluginFilePath = FoundPlugins.Last();
		IPluginManager::Get().AddToPluginsList(Package->DLCPluginFilePath);

		Package->DLCPlugin = IPluginManager::Get().FindPlugin(PluginName);
		checkf(Package->DLCPlugin.IsValid(), TEXT("failed to find DLC plugin %s"), *PluginName);

		// ensure the plugin is mounted - will also load the shader code library if there is one
		if (Package->DLCPlugin->GetDescriptor().bExplicitlyLoaded)
		{
			IPluginManager::Get().MountExplicitlyLoadedPlugin_FromFileName(Package->DLCPluginFilePath);
		}
		else
		{
			UE_LOGF(LogPlatformDLC, Warning, "Plugin %ls is not marked for explicit loading - unmounting DLC %ls may fail", *PluginName, *Package->DLCName.ToString());
			IPluginManager::Get().MountNewlyCreatedPlugin(PluginName);
		}

		// load the asset registry for the new plugin
		if (Package->DLCPlugin->CanContainContent())
		{
			FString AssetRegistryFileName = Package->DLCPlugin->GetBaseDir() / TEXT("AssetRegistry.bin");
			FPaths::MakePathRelativeTo(AssetRegistryFileName, *Package->MountPath);
			AssetRegistryFileName = TEXT("../../..") / AssetRegistryFileName;

			FAssetRegistryState PluginAssetRegistry;
			if (FAssetRegistryState::LoadFromDisk(*AssetRegistryFileName, FAssetRegistryLoadOptions(), PluginAssetRegistry))
			{
				IAssetRegistry::GetChecked().AppendState(PluginAssetRegistry);
			}
			else
			{
				UE_LOGF(LogPlatformDLC, Error, "Failed to load plugin asset registry state %ls", *AssetRegistryFileName);
			}
		}
	}

	// all done
	Package->MountState = EMountState::Mounted;
	ProcessingState = EProcessingState::Mount_Completed;
}


void FGDKPlatformDLC::Mount_Completed_GameThread( TSharedPtr<FPackageData> Package )
{
	// signal callback
	bool bSuccess = (Package->MountState == EMountState::Mounted);
	PostNotification(Package->DLCName, ENotification::Mounted, bSuccess);

	ProcessingState = EProcessingState::Idle;
}


void FGDKPlatformDLC::LicenseLost_GameThread( TSharedPtr<FPackageData> Package )
{
	// close the existing license
	if (Package->DLCLicense != nullptr)
	{
		XStoreUnregisterPackageLicenseLost( Package->DLCLicense, Package->PackageLicenseLostToken, true);
		XStoreCloseLicenseHandle(Package->DLCLicense);
		Package->DLCLicense = nullptr;
	}
	Package->LicenseState = ELicenseState::LicenseLost;

	// signal that the license has been lost. it's up to the title whether they want to retry/unmount/terminate etc.
	PostNotification(Package->DLCName, ENotification::Entitlement, false);
}



bool FGDKPlatformDLC::Unmount( FName DLCName )
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TSharedPtr<FGDKPlatformDLC::FPackageData> Package = FindDLC(DLCName);
	if (!Package.IsValid())
	{
		UE_LOGF(LogPlatformDLC, Error, "DLC %ls not known", *DLCName.ToString());
		return false;	
	}
	
	bool bSuccess = false;
	// @todo: make this async

	FScopeLock Lock(&PackageDataLock);

	if (Package->DLCHandle != nullptr)
	{
		// attempt to unregister the plugin as much as possible
		if (bAutoMountPlugin && bAutoMountPakFiles && Package->DLCPlugin.IsValid())
		{
			const FString& PluginName = DLCNameToPluginNameMap.FindChecked(Package->DLCName);

			FText FailureReason;
			if (Package->DLCPlugin->GetDescriptor().bExplicitlyLoaded)
			{
				bool bResult = IPluginManager::Get().UnmountExplicitlyLoadedPlugin(PluginName, &FailureReason);
				UE_CLOGF(!bResult, LogPlatformDLC, Warning, "Failed to unmount plugin %ls : %ls", *PluginName, *FailureReason.ToString());

				IPluginManager::Get().RemoveFromPluginsList( Package->DLCPluginFilePath );
			}
			else
			{
				// not possible to properly unmount, so just simulate the results
				IPluginManager::Get().OnPluginUnmounted().Broadcast(*Package->DLCPlugin);
				FPackageName::UnRegisterMountPoint( Package->DLCPlugin->GetMountedAssetPath(), Package->DLCPlugin->GetContentDir() );
			}
			Package->DLCPlugin.Reset();
			Package->DLCPluginFilePath.Reset();
		}

		// unmount all pak files
		if (bAutoMountPakFiles)
		{
			TArray<FString> PakFileNames;
			IPlatformFile::GetPlatformPhysical().FindFilesRecursively(PakFileNames, *Package->MountPath, TEXT(".pak") );
			for (const FString& PakFileName : PakFileNames)
			{
				if (IGDKPackageManifestModule::Get().IsPakFileInstalled(Package->DLCHandle, PakFileName))
				{
					bool bUnmounted = FCoreDelegates::OnUnmountPak.IsBound() ? FCoreDelegates::OnUnmountPak.Execute(PakFileName) : false;
					if (bUnmounted)
					{
						UE_LOGF(LogPlatformDLC, Log, "Successfully unmounted %ls", *PakFileName);
					}
					else
					{
						UE_LOGF(LogPlatformDLC, Log, "Failed to unmount %ls", *PakFileName);
					}
				}
			}
		}

		IGDKPackageManifestModule::Get().UnloadDLC(Package->DLCHandle);

		// unmount the package
		XPackageCloseMountHandle(Package->DLCHandle);
		Package->DLCHandle = nullptr;

		Package->MountState = EMountState::Unmounted;

		bSuccess = true;
	}

	PostNotification(DLCName, ENotification::Unmounted, bSuccess);
	return true;
}


void FGDKPlatformDLC::AddToTicker()
{
	if (!TickHandle.IsValid())
	{
		FTSTicker& Ticker = FTSTicker::GetCoreTicker();

		// Register delegate for ticker callback
		FTickerDelegate TickDelegate = FTickerDelegate::CreateRaw(this, &FGDKPlatformDLC::Tick);
		TickHandle = Ticker.AddTicker(TickDelegate, 0.1f);
	}
}

void FGDKPlatformDLC::RemoveFromTicker()
{
	FTSTicker& Ticker = FTSTicker::GetCoreTicker();

	// Unregister ticker delegate
	if (TickHandle.IsValid())
	{
		Ticker.RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
}



void FGDKPlatformDLC::OnInstallationProgressChanged(void* Context, XPackageInstallationMonitorHandle Monitor)
{
	TSharedPtr<FPackageData> Package = ((FPackageData::FContext*)Context)->Package.Pin();
	if (Package.IsValid())
	{
		XPackageInstallationProgress Progress;
		XPackageGetInstallationProgress(Monitor, &Progress);
		Package->CachedInstalledBytes = Progress.installedBytes;
		Package->CachedTotalBytes = Progress.totalBytes;

		const float Percent = (Progress.totalBytes > 0) ? 100.0f * (float)Progress.installedBytes / (float)Progress.totalBytes : 0.0f;
		UE_LOGF(LogPlatformDLC, Verbose, "DLC %ls download progress changed: %20llu / %20llu bytes  (%6.2f%%)  %ls", *Package->DLCName.ToString(), Progress.installedBytes, Progress.totalBytes, Percent, Progress.completed ? TEXT("Completed") : TEXT(""));
	}
}


void FGDKPlatformDLC::OnPackageLicenseLost(void* Context)
{
	TSharedPtr<FPackageData> Package = ((FPackageData::FContext*)Context)->Package.Pin();
	{
		FScopeLock Lock(&Package->Owner->PackageDataLock);
		Package->Owner->LostLicensePackages.AddUnique(Package);
		Package->Owner->AddToTicker();

		XStoreUnregisterPackageLicenseLost( Package->DLCLicense, Package->PackageLicenseLostToken, false);
	}
}


void FGDKPlatformDLC::SetStoreUser( FPlatformUserId PlatformUserId )
{
	FGDKUserHandle NewStoreUser = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(PlatformUserId);
	SetStoreUserHandle(NewStoreUser);
}


FPlatformUserId FGDKPlatformDLC::GetStoreUser() const
{
	return CurrentStoreUserId;
}


void FGDKPlatformDLC::SetStoreUserHandle(const FGDKUserHandle& NewStoreUser)
{
	if (NewStoreUser != CurrentStoreUser)
	{
		CurrentStoreUser = NewStoreUser;
		CurrentStoreUserId = IGDKRuntimeModule::Get().GetPlatformIdByUserHandle(NewStoreUser);

		if (CurrentStoreContext != nullptr)
		{
			XStoreCloseContextHandle(CurrentStoreContext);
			CurrentStoreContext = nullptr;
		}

		bPendingProductsRequery = true;
		AddToTicker();
	}
}


void FGDKPlatformDLC::OnUserLoginChanged(bool bSignedIn, int32 ChangedUserIndex, int32)
{
	if (!IsInitialized())
	{
		return;
	}

	FPlatformUserId ChangedUserId = FPlatformUserId::CreateFromInternalId(ChangedUserIndex);

	// if we have no store user, adopt the newly signed-in user
	if (bSignedIn && !CurrentStoreUser.IsValid())
	{
		FGDKUserHandle NewUser = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(ChangedUserId);
		SetStoreUserHandle(NewUser);
		return;
	}

	// we only care if the current store user signs out
	if (bSignedIn || ChangedUserId != CurrentStoreUserId)
	{
		return;
	}

	// try to find the next best store user as a fallback
	FGDKUserHandle NewStoreUser;
	for (const FGDKUserHandle& Handle : IGDKRuntimeModule::Get().GetAllUserHandles())
	{
		if (Handle != CurrentStoreUser)
		{
			NewStoreUser = Handle;
			break;
		}
	}

	// update the store user, if any
	SetStoreUserHandle(NewStoreUser);
}


bool FGDKPlatformDLC::EnsureValidStoreContext()
{
	if (CurrentStoreUser.IsValid() && CurrentStoreContext == nullptr)
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // XStoreCreateContext is not safe to call on a time-sensitive thread

		HRESULT hResult = XStoreCreateContext(CurrentStoreUser, &CurrentStoreContext);
		if (FAILED(hResult))
		{
			UE_LOGF(LogPlatformDLC, Error, "Failed to create store context for user %ls : 0x%X", *LexToString(CurrentStoreUser), hResult);
			return false;
		}
	}

	return (CurrentStoreContext != nullptr);
}


bool FGDKPlatformDLC::ProcessQueryProductsResult(XAsyncBlock* Block)
{
	XStoreProductQueryHandle QueryHandle = nullptr;
	HRESULT hResult = XStoreQueryProductsResult(Block, &QueryHandle);
	if (FAILED(hResult))
	{
		UE_LOG(LogPlatformDLC, Error, TEXT("Failed to query DLC products: 0x%X"), hResult);
		return false;
	}

	auto EnumeratePackageCallback = [](const XStoreProduct* Product, void* Context)
	{
		FGDKPlatformDLC* This = (FGDKPlatformDLC*)Context;

		TSharedPtr<FPackageData> Package = This->FindDLCByStoreId( Product->storeId );
		if (Package.IsValid())
		{
			Package->bIsPurchased = Product->isInUserCollection; // @todo: do we need to enumerate Product->skus ?
			UE_LOG(LogPlatformDLC, Log, TEXT("found available DLC product %-32hs %18hs : %s"), Product->title, Product->storeId, Product->isInUserCollection?TEXT("owned"):TEXT("not owned"));
		}
		return true;
	};

	hResult = XStoreEnumerateProductsQuery(QueryHandle, this, EnumeratePackageCallback);
	XStoreCloseProductsQueryHandle(QueryHandle);
	if (FAILED(hResult))
	{
		UE_LOG(LogPlatformDLC, Error, TEXT("Failed to enumerate DLC products: 0x%X"), hResult);
		return false;
	}

	return true;
}



TSharedPtr<FGDKPlatformDLC::FPackageData> FGDKPlatformDLC::FindDLC( FName DLCName )
{
	FScopeLock Lock(&PackageDataLock);

	for (TSharedPtr<FPackageData> PackageData : AllPackages)
	{
		if (PackageData->DLCName == DLCName)
		{
			return PackageData;
		}
	}

	return nullptr;
}

TSharedPtr<const FGDKPlatformDLC::FPackageData> FGDKPlatformDLC::FindDLC( FName DLCName ) const
{
	TSharedPtr<FPackageData> Result = const_cast<FGDKPlatformDLC*>(this)->FindDLC(DLCName);
	return Result;
}

TSharedPtr<FGDKPlatformDLC::FPackageData> FGDKPlatformDLC::FindDLCByStoreId( const char* StoreId )
{
	FScopeLock Lock(&PackageDataLock);
	for (const TSharedPtr<FPackageData>& Package : AllPackages)
	{
		if (FCStringAnsi::Strcmp(Package->StoreId, StoreId) == 0)
		{
			return Package;
		}
	}

	return nullptr;
}

bool FGDKPlatformDLC::HasEntitlement( FName DLCName ) const
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TSharedPtr<const FPackageData> Package = FindDLC(DLCName);
	return Package.IsValid() && (Package->LicenseState == ELicenseState::Licensed);
}

IPlatformDLC::EState FGDKPlatformDLC::GetState( FName DLCName ) const
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TSharedPtr<const FPackageData> Package = FindDLC(DLCName);
	if (Package.IsValid())
	{
		if (Package->MountState == EMountState::Mounted)
		{
			return EState::Mounted;
		}
		else if (Package->DownloadState == EDownloadState::Downloaded)
		{
			return EState::Downloaded;
		}
		else if (Package->DownloadState == EDownloadState::Downloading)
		{
			return EState::Downloading;
		}
	}

	return EState::NotInstalled;
}

bool FGDKPlatformDLC::Mount( FName DLCName )
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TSharedPtr<FPackageData> Package = FindDLC(DLCName);
	if (!Package.IsValid())
	{
		return false;
	}

	if (Package->DownloadState != EDownloadState::Downloaded)
	{
		UE_LOGF(LogPlatformDLC, Error, "DLC %ls must be downloaded before it can be mounted", *DLCName.ToString());
		return false;
	}

	Package->bAllowPurchasePrompt = true;
	return AddToPendingPackages(Package);
}


bool FGDKPlatformDLC::GetDownloadSize( FName DLCName, uint64& OutCurrentInstallSize, uint64& OutFullInstallSize ) const
{
	OutCurrentInstallSize = 0;
	OutFullInstallSize = 0;

	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TSharedPtr<const FPackageData> Package = FindDLC(DLCName);
	if (!Package.IsValid())
	{
		return false;
	}

	// CachedInstalledBytes is 0 before a download has started, and CachedTotalBytes is UINT64_MAX until the GDK resolves the package size
	if (Package->CachedInstalledBytes == 0 && Package->CachedTotalBytes == UINT64_MAX)
	{
		return false;
	}

	OutCurrentInstallSize = Package->CachedInstalledBytes;
	OutFullInstallSize = Package->CachedTotalBytes;
	return true;
}

FString FGDKPlatformDLC::GetRootDirectory( FName DLCName ) const
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TSharedPtr<const FPackageData> Package = FindDLC(DLCName);
	if (ensure(Package.IsValid() && Package->MountState == EMountState::Mounted))
	{
		return Package->MountPath;
	}

	return FString();
}

TArray<FName> FGDKPlatformDLC::GetAllDLCNames() const
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	FScopeLock Lock(&PackageDataLock);

	TArray<FName> DLCNames;
	DLCNameToStoreIdMap.GetKeys(DLCNames);

	return MoveTemp(DLCNames);
}

TArray<FName> FGDKPlatformDLC::GetMountedDLCNames() const
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TArray<FName> DLCNames;
	for (const FName& DLCName : GetAllDLCNames())
	{
		if (GetState(DLCName) == EState::Mounted)
		{
			DLCNames.Add(DLCName);
		}
	}

	return MoveTemp(DLCNames);
}

FString FGDKPlatformDLC::GetStoreId( FName DLCName ) const
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	const FString* StoreIdPtr = DLCNameToStoreIdMap.Find(DLCName);
	if (StoreIdPtr != nullptr)
	{
		return *StoreIdPtr;
	}

	return FString();
}




FGDKPlatformDLC::FPackageData::FPackageData(FGDKPlatformDLC* InOwner, const FName& InDLCName, const FString& InStoreId)
	: Owner(InOwner)
	, DLCName(InDLCName)
{
	FCStringAnsi::Strncpy(StoreId, TCHAR_TO_UTF8(*InStoreId), STORE_SKU_ID_SIZE);
	PackageName = InDLCName.ToString(); // placeholder until the display name is known
}




FGDKPlatformDLC::FPackageData::~FPackageData()
{
	DestroyInstallationMonitor();

	if (DLCHandle != nullptr)
	{
		XPackageCloseMountHandle(DLCHandle);
		DLCHandle = nullptr;
	}

	if (DLCLicense != nullptr)
	{
		XStoreUnregisterPackageLicenseLost( DLCLicense, PackageLicenseLostToken, true);
		XStoreCloseLicenseHandle(DLCLicense);
		DLCLicense = nullptr;
	}
}

void FGDKPlatformDLC::FPackageData::OnDownloading(const char* InPackageIdentifier)
{
	// store package details
	FCStringAnsi::Strncpy(PackageIdentifier, InPackageIdentifier, XPACKAGE_IDENTIFIER_MAX_LENGTH);
	
	DownloadState = EDownloadState::Downloading;
	CreateInstallationMonitor(PackageIdentifier);
}


void FGDKPlatformDLC::FPackageData::OnDownloaded(const XPackageDetails& InDetails)
{
	// store package details
	FCStringAnsi::Strncpy(PackageIdentifier, InDetails.packageIdentifier, XPACKAGE_IDENTIFIER_MAX_LENGTH);
	bIsAgeRestricted = InDetails.ageRestricted;
	if (InDetails.displayName[0] != '\0')
	{
		PackageName = UTF8_TO_TCHAR(InDetails.displayName);
	}

	DownloadState = EDownloadState::Downloaded;
	DestroyInstallationMonitor();
}


void FGDKPlatformDLC::FPackageData::OnUninstalled()
{
	// clear package details 
	PackageIdentifier[0] = '\0';
	MountPath.Reset();
	DLCPlugin.Reset();
	DLCPluginFilePath.Reset();
	CachedInstalledBytes = 0; // not clearing CachedTotalBytes too because that will not change on a subsequent re-download
	MountState = EMountState::NotMounted;
	
	DownloadState = EDownloadState::NotDownloaded;
	DestroyInstallationMonitor();	
}


void FGDKPlatformDLC::FPackageData::CreateInstallationMonitor(const char* Identifier)
{
	if (InstallationMonitorHandle == nullptr)
	{
		HRESULT hResult = XPackageCreateInstallationMonitor(Identifier, 0, nullptr, 500, FGDKAsyncTaskQueue::GetGenericQueue(), &InstallationMonitorHandle);
		if (SUCCEEDED(hResult))
		{
			XPackageInstallationProgress Progress;
			XPackageGetInstallationProgress(InstallationMonitorHandle, &Progress);
			CachedInstalledBytes = Progress.installedBytes;
			CachedTotalBytes = Progress.totalBytes;

			hResult = XPackageRegisterInstallationProgressChanged(InstallationMonitorHandle, &Context, &FGDKPlatformDLC::OnInstallationProgressChanged, &InstallationProgressToken);
			UE_CLOGF(FAILED(hResult), LogPlatformDLC, Warning, "Failed to register installation progress callback for %ls : 0x%X", *PackageName, hResult);
		}
		else
		{
			UE_LOGF(LogPlatformDLC, Warning, "Failed to create installation monitor for %ls : 0x%X", *PackageName, hResult);
		}
	}
}


void FGDKPlatformDLC::FPackageData::DestroyInstallationMonitor()
{
	if (InstallationMonitorHandle != nullptr)
	{
		XPackageUnregisterInstallationProgressChanged(InstallationMonitorHandle, InstallationProgressToken, true);
		InstallationProgressToken = {};
		
		XPackageCloseInstallationMonitorHandle(InstallationMonitorHandle);
		InstallationMonitorHandle = nullptr;
	}
}





bool FGDKPlatformDLC::Uninstall( FName DLCName )
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (debug only) XPackageUninstallPackage is not safe to call on a time-sensitive thread

	TSharedPtr<FGDKPlatformDLC::FPackageData> Package = FindDLC(DLCName);
	if (Package.IsValid())
	{
		if (Package->DLCHandle == nullptr)
		{
			// we must release the license before uninstalling. if we hold this and the DLC is uninstalled & reinstalled, we may get device I/O errors
			if (Package->DLCLicense != nullptr)
			{
				XStoreUnregisterPackageLicenseLost( Package->DLCLicense, Package->PackageLicenseLostToken, true);
				XStoreCloseLicenseHandle(Package->DLCLicense);
				Package->DLCLicense = nullptr;
				Package->LicenseState = ELicenseState::Unlicensed;
				PostNotification(Package->DLCName, ENotification::Entitlement, false);
			}

			bool bSuccess = XPackageUninstallPackage(Package->PackageIdentifier);
			if (bSuccess)
			{
				FScopeLock Lock(&PackageDataLock);
				Package->OnUninstalled();
			}
			PostNotification(Package->DLCName, ENotification::Uninstalled, bSuccess);
			return true;
		}
		else
		{
			UE_LOGF(LogPlatformDLC, Error, "DLC %ls must be unmounted before it can be uninstalled", *DLCName.ToString());
			return false;
		}

	}

	return false;
}

bool FGDKPlatformDLC::Download( FName DLCName )
{
	checkf(IsInitialized(), TEXT("Platform DLC is not initialized"));

	TSharedPtr<FGDKPlatformDLC::FPackageData> Package = FindDLC(DLCName);
	if (!Package.IsValid())
	{
		return false;
	}

	if (Package->DownloadState == EDownloadState::Downloaded)
	{
		PostNotification(DLCName, ENotification::Downloaded, true);
		if (bAutoMountDLC)
		{
			Package->bAllowPurchasePrompt = true;
			return AddToPendingPackages(Package);
		}
		else
		{
			return true;
		}
	}

	if (Package->DownloadState == EDownloadState::Downloading)
	{
		// attempt to prioritize this DLC (no way to do this without using the chunk selector. All DLCs we generate have this chunk)
		XPackageChunkSelector Selector { .type = XPackageChunkSelectorType::Chunk, .chunkId = 2 };
		HRESULT hResult = XPackageChangeChunkInstallOrder(Package->PackageIdentifier, 1, &Selector);
		UE_CLOG(FAILED(hResult), LogPlatformDLC, Warning, TEXT("Cannot change DLC install order 0x%X"), hResult);

		return true;
	}

	// make sure there is a store context
	if (!EnsureValidStoreContext())
	{
		return false;
	}

	if (!Package->bIsPurchased)
	{
		Package->bAllowPurchasePrompt = true;
		return AddToPendingPackages(Package);
	}
	else
	{
		return AsyncDownloadPackage(Package);
	}
}

bool FGDKPlatformDLC::AsyncDownloadPackage( TSharedPtr<FPackageData> Package )
{
	// kick off the async download
	HRESULT hResult = AsyncGDKTask(
		Package->DownloadAsyncTask,
		[this, Package]( XAsyncBlock* AsyncBlock )
		{
			const char* StoreIds[] = 
			{
				Package->StoreId
			};

			return XStoreDownloadAndInstallPackagesAsync( CurrentStoreContext, StoreIds, 1, AsyncBlock );
		},
		[this, Package]( XAsyncBlock* AsyncBlock )
		{
			// early out if the DLC system is being destroyed
			bool bCancel = (ProcessingState == EProcessingState::Destroying);
			if (bCancel)
			{
				return;
			}
		
			HRESULT hResult;

			uint32 NumPackages = 0;
			hResult = XStoreDownloadAndInstallPackagesResultCount( AsyncBlock, &NumPackages );
			if (FAILED(hResult) || NumPackages == 0)
			{
				UE_LOGF(LogPlatformDLC, Error, "failed to get installing packages count for %ls : 0x%X", *Package->DLCName.ToString(), hResult );
				PostNotification(Package->DLCName, ENotification::Downloaded, false);
				return;
			}
			if (NumPackages > 1) // download will still trigger XPackageRegisterPackageInstalled
			{
				UE_LOGF(LogPlatformDLC, Error, "expected to get 1 package being downloaded for %ls, but got %d. Installation progress will not be tracked", *Package->DLCName.ToString(), NumPackages );
				return;
			}

			char PackageIdentifier[XPACKAGE_IDENTIFIER_MAX_LENGTH];
			hResult = XStoreDownloadAndInstallPackagesResult( AsyncBlock, 1, &PackageIdentifier );
			if (FAILED(hResult))
			{
				UE_LOGF(LogPlatformDLC, Error, "failed to get installing package identifier for %ls : 0x%X", *Package->DLCName.ToString(), hResult );
				PostNotification(Package->DLCName, ENotification::Downloaded, false);
				return;
			}

			// the download was successfully started. we must have a valid license for this before we can mount it
			UE_LOGF(LogPlatformDLC, Log, "Downloading package %ls for DLC %ls", UTF8_TO_TCHAR(PackageIdentifier), *Package->DLCName.ToString() );
			{
				FScopeLock Lock(&PackageDataLock);
				Package->OnDownloading(PackageIdentifier);
			}
		},
		FGDKAsyncTaskQueue::GetBackgroundTaskQueue() // completion callback happens on a background thread
	);

	return SUCCEEDED(hResult);
}

#if !UE_BUILD_SHIPPING
FString FGDKPlatformDLC::Debug_GetStateDescription() const
{
	return FString::Printf( TEXT("GDK DLC.  Store User: %-3d %-16s   Processing State: %s"), CurrentStoreUserId.GetInternalId(), *IGDKRuntimeModule::Get().GetGamertag(CurrentStoreUser), *LexToString(ProcessingState) );

}

bool FGDKPlatformDLC::Debug_GetPackageDescription( int PackageIndex, FString& OutDescription, FColor& OutColor ) const
{
	TSharedPtr<FPackageData> Package;
	{
		FScopeLock Lock(&PackageDataLock);
		if (!AllPackages.IsValidIndex(PackageIndex))
		{
			return false;
		}
		Package = AllPackages[PackageIndex];
	}

	if (Package->DownloadState == EDownloadState::NotDownloaded)
	{
		OutColor = FColor::Silver;
	}
	else if (Package->DownloadState == EDownloadState::Downloading)
	{
		OutColor = FColor::Cyan;
	}
	else if (Package->MountState == EMountState::NotMounted || Package->MountState == EMountState::Unmounted)
	{
		OutColor = FColor::Silver;
	}
	else if (Package->MountState == EMountState::Mounted)
	{
		bool bIsLicensed = (Package->LicenseState == ELicenseState::Licensed);
		OutColor = bIsLicensed ? FColor::Green : FColor::Yellow;
	}
	else if (Package->MountState < EMountState::Mounted)
	{
		OutColor = FColor::Yellow;
	}
	else
	{
		OutColor = FColor::Red;
	}

	FString DownloadState;
	if (Package->DownloadState == EDownloadState::Downloading && Package->CachedTotalBytes > 0 && Package->CachedTotalBytes != UINT64_MAX)
	{
		const float Percent = 100.0f * (float)Package->CachedInstalledBytes / (float)Package->CachedTotalBytes;
		DownloadState = FString::Printf(TEXT("Downloading(%.0f%%)"), Percent);
	}
	else
	{
		DownloadState = LexToString(Package->DownloadState);
	}

	FString InUserCollection = Package->bIsPurchased ? TEXT("Purchased") : TEXT("Not Purchased");

	OutDescription = FString::Printf(TEXT(".......%-30s %-20s %-15s %-15s %-15s (%s)"), *Package->DLCName.ToString(), *DownloadState, *InUserCollection, *LexToString(Package->MountState), *LexToString(Package->LicenseState), *Package->PackageName );
	return true;
}
#endif //UE_BUILD_SHIPPING


FString LexToString( FGDKPlatformDLC::EMountState MountState )
{
	switch (MountState)
	{
		case FGDKPlatformDLC::EMountState::NotMounted:              return TEXT("NotMounted");
		case FGDKPlatformDLC::EMountState::Mounted:                 return TEXT("Mounted");
		case FGDKPlatformDLC::EMountState::Unmounted:               return TEXT("Unmounted");
		case FGDKPlatformDLC::EMountState::Ignored_AgeRestricted:   return TEXT("Ignored(AgeRestricted)");
		default:                                                    return TEXT("Invalid");
	}
}

FString LexToString( FGDKPlatformDLC::ELicenseState LicenseState )
{
	switch(LicenseState)
	{
		case FGDKPlatformDLC::ELicenseState::Unlicensed:            return TEXT("Unlicensed");
		case FGDKPlatformDLC::ELicenseState::Licensed:              return TEXT("Licensed");
		case FGDKPlatformDLC::ELicenseState::LicenseLost:           return TEXT("LicenseLost");
		default:                                                    return TEXT("Invalid");
	}
}

FString LexToString( FGDKPlatformDLC::EProcessingState ProcessingState )
{
	switch(ProcessingState)
	{
		case FGDKPlatformDLC::EProcessingState::Initializing:                return TEXT("Initializing");
		case FGDKPlatformDLC::EProcessingState::Idle:                        return TEXT("Idle");
		case FGDKPlatformDLC::EProcessingState::QueryingProducts:            return TEXT("QueryingProducts");
		
		case FGDKPlatformDLC::EProcessingState::Purchase_QueryingProduct:    return TEXT("Purchase(QueryingProduct)");
		case FGDKPlatformDLC::EProcessingState::Purchase_ShowingPurchaseUI:  return TEXT("Purchase(ShowingPurchaseUI)");
		case FGDKPlatformDLC::EProcessingState::Purchase_DownloadRequested:  return TEXT("Purchase(DownloadRequested)");
		case FGDKPlatformDLC::EProcessingState::Purchase_Completed:          return TEXT("Purchase(Completed)");

		case FGDKPlatformDLC::EProcessingState::Mount_AcquiringLicense:      return TEXT("Mounting(AcquiringLicense)");
		case FGDKPlatformDLC::EProcessingState::Mount_MountingPackage:       return TEXT("Mounting(MountingPackage)");
		case FGDKPlatformDLC::EProcessingState::Mount_CachingData:           return TEXT("Mounting(CachingData)");
		case FGDKPlatformDLC::EProcessingState::Mount_MountingPlugin:        return TEXT("Mounting(MountingPlugin)");
		case FGDKPlatformDLC::EProcessingState::Mount_Completed:             return TEXT("Mounting(Completed)");

		case FGDKPlatformDLC::EProcessingState::Destroying:                  return TEXT("Destroying");
		default:                                                             return TEXT("Invalid");
	}
}

FString LexToString( FGDKPlatformDLC::EDownloadState DownloadState )
{
	switch(DownloadState)
	{
		case FGDKPlatformDLC::EDownloadState::NotDownloaded:  return TEXT("NotDownloaded");
		case FGDKPlatformDLC::EDownloadState::Downloading:    return TEXT("Downloading");
		case FGDKPlatformDLC::EDownloadState::Downloaded:     return TEXT("Downloaded");
		default:                                              return TEXT("Invalid");
	}
}

#endif //WITH_GRDK
