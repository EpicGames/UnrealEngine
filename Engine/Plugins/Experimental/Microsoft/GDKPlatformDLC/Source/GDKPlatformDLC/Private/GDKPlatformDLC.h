// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlatformDLC.h"
#include "GDKRuntimeModule.h"
#if WITH_GRDK
#include "GDKPlatformDLCModule.h"
#include "GDKTaskQueueHelpers.h"
#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Ticker.h"
#include "Math/Color.h"
#include "Misc/Optional.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XPackage.h>
#include <XStore.h>
#include <atomic>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

class FGDKPlatformDLC : public FGenericPlatformDLC
{
public:
	FGDKPlatformDLC();
	virtual ~FGDKPlatformDLC();

	// IPlatformDLC implementation
	virtual void Shutdown() override;

	virtual bool HasEntitlement( FName DLCName ) const override;
	virtual EState GetState( FName DLCName ) const override;

	virtual bool Mount( FName DLCName ) override;
	virtual bool Unmount( FName DLCName ) override;
	virtual bool Download( FName DLCName ) override;
	virtual bool Uninstall( FName DLCName ) override;

	virtual bool GetDownloadSize( FName DLCName, uint64& OutCurrentInstallSize, uint64& OutFullInstallSize ) const override;
	virtual FString GetRootDirectory( FName DLCName ) const override;
	virtual TArray<FName> GetAllDLCNames() const override;
	virtual TArray<FName> GetMountedDLCNames() const override;
	virtual FString GetStoreId( FName DLCName ) const override;

	virtual void SetStoreUser( FPlatformUserId PlatformUserId ) override;
	virtual FPlatformUserId GetStoreUser() const override;

	// support for debug rendering
#if !UE_BUILD_SHIPPING
	FString Debug_GetStateDescription() const;
	bool Debug_GetPackageDescription( int PackageIndex, FString& OutDescription, FColor& OutColor ) const;
#endif

private:
	void SetStoreUserHandle(const FGDKUserHandle& NewStoreUser);


	enum class EMountState : uint8
	{
		NotMounted,
		Mounted,
		Unmounted,
		Ignored_AgeRestricted,
	};
	enum class ELicenseState : uint8
	{
		Unlicensed,
		Licensed,
		LicenseLost,
	};
	enum class EDownloadState : uint8
	{
		NotDownloaded,
		Downloading,
		Downloaded,
	};

	struct FPackageData
	{
		FPackageData(FGDKPlatformDLC* InOwner, const FName& InDLCName, const FString& InStoreId);
		~FPackageData();

		void OnDownloading(const char* InPackageIdentifier);
		void OnDownloaded(const XPackageDetails& InDetails);
		void OnUninstalled();

		void CreateInstallationMonitor(const char* Identifier);
		void DestroyInstallationMonitor();

		char                              PackageIdentifier[XPACKAGE_IDENTIFIER_MAX_LENGTH] = {};
		char                              StoreId[STORE_SKU_ID_SIZE] = {};
		std::atomic<EMountState>          MountState = EMountState::NotMounted;
		std::atomic<ELicenseState>        LicenseState = ELicenseState::Unlicensed;
		std::atomic<EDownloadState>       DownloadState = EDownloadState::NotDownloaded;
		FString                           PackageName;
		FString                           MountPath;
		FGDKPlatformDLC*                  Owner = nullptr;
		bool                              bAllowPurchasePrompt = false;
		std::atomic<bool>                 bIsPurchased = false;
		bool                              bIsAgeRestricted = false;
		FName                             DLCName;
		XPackageMountHandle               DLCHandle = nullptr;
		XStoreLicenseHandle               DLCLicense = nullptr;
		XTaskQueueRegistrationToken       PackageLicenseLostToken = {};
		TSharedPtr<class IPlugin>         DLCPlugin;
		FString                           DLCPluginFilePath;
		FGDKAsyncTaskMonitor              DownloadAsyncTask;
		XPackageInstallationMonitorHandle InstallationMonitorHandle = nullptr;
		XTaskQueueRegistrationToken       InstallationProgressToken = {};
		std::atomic<uint64>               CachedInstalledBytes = 0;
		std::atomic<uint64>               CachedTotalBytes = 0;

		// context used for GDK callbacks that only accept void*
		struct FContext
		{
			TWeakPtr<FPackageData> Package;
		};
		FContext Context;
	};
	TArray<TSharedPtr<FPackageData>> AllPackages;
	TArray<const char*>              AllStoreIds; // points to StoreId in AllPackages 
	TMap<FName, FString>             DLCNameToStoreIdMap;
	TMap<FName, FString>             DLCNameToPluginNameMap;
	mutable FCriticalSection         PackageDataLock;
	TFuture<void>                    InitFuture;
	XTaskQueueRegistrationToken      PackageInstalledToken;
	FDelegateHandle                  UserLoginChangedDelegateHandle;

	void AsyncInitDLC(const XPackageDetails& Details, bool bMount, bool bNotify);
	TSharedPtr<FPackageData> FindDLC( FName DLCName );
	TSharedPtr<const FPackageData> FindDLC( FName DLCName ) const;
	TSharedPtr<FPackageData> FindDLCByStoreId( const char* StoreId );

	// DLC processing
	enum class EProcessingState : uint8
	{
		Initializing,
		Idle,
		QueryingProducts,

		Purchase_ShowingPurchaseUI,
		Purchase_QueryingProduct,
		Purchase_DownloadRequested,
		Purchase_Completed,

		Mount_AcquiringLicense,
		Mount_MountingPackage,
		Mount_CachingData,
		Mount_MountingPlugin,
		Mount_Completed,

		Destroying,
	};
	std::atomic<EProcessingState>    ProcessingState = EProcessingState::Idle;
	bool                             bPendingProductsRequery = false;
	TArray<TSharedPtr<FPackageData>> PendingPackages;
	TArray<TSharedPtr<FPackageData>> LostLicensePackages;
	TSharedPtr<FPackageData>         CurrentProcessingPackage;
	FGDKAsyncTaskMonitor             CurrentProcessingAsyncTask;

	virtual void BeginInitializeInternal() override;

	bool AddToPendingPackages( TSharedPtr<FPackageData> Package );
	void AsyncQueryingProducts_GameThread();
	void BeginProcessingPackage_GameThread( TSharedPtr<FPackageData> Package );

	void Purchase_AsyncShowPurchaseUI_GameThread( TSharedPtr<FPackageData> Package );
	void Purchase_AsyncQueryingProduct_GameThread( TSharedPtr<FPackageData> Package );
	void Purchase_Completed_GameThread( TSharedPtr<FPackageData> Package );

	void Mount_AsyncAcquireLicense_GameThread( TSharedPtr<FPackageData> Package );
	void Mount_AsyncMountPackage_GameThread( TSharedPtr<FPackageData> Package );
	void Mount_CachePackageData_BackgroundThread( TSharedPtr<FPackageData> Package );
	void Mount_MountPlugin_GameThread( TSharedPtr<FPackageData> Package );
	void Mount_Completed_GameThread( TSharedPtr<FPackageData> Package );
	void LicenseLost_GameThread( TSharedPtr<FPackageData> Package );

	bool AsyncDownloadPackage( TSharedPtr<FPackageData> Package );

	// store
	XStoreContextHandle CurrentStoreContext;
	FGDKUserHandle      CurrentStoreUser;
	FPlatformUserId     CurrentStoreUserId;
	bool EnsureValidStoreContext();
	bool ProcessQueryProductsResult(XAsyncBlock* Block);
	static void OnPackageLicenseLost(void* Context);
	static void OnInstallationProgressChanged(void* Context, XPackageInstallationMonitorHandle Monitor);
	void OnUserLoginChanged(bool bSignedIn, int32 ChangedUserIndex, int32 ControllerId);

	// configuration
	bool bAutoMountDLC = true;            // Whether to automatically mount any new DLC that is downloaded once the application is up and running.
	bool bAutoMountExistingDLC = true;    // Whether to automatically mount any already-downloaded DLC on system startup.
	bool bAutoMountPakFiles = true;       // Whether to automatically mount pak files
	bool bAutoMountPlugin = true;         // Whether to automatically mount the DLC plugin and asset registry etc. (requires bAutoMountPakFiles)
	bool bAutoMountAgeRestricted = false; // Whether to attempt to mount age-restricted DLC. This will trigger a parental controls consent prompt.

	// ticker
	FTSTicker::FDelegateHandle TickHandle;
	void AddToTicker();
	void RemoveFromTicker();
	bool Tick( float DeltaSeconds );

	// internal LexToString
	friend FString LexToString( EMountState MountState );
	friend FString LexToString( ELicenseState LicenseState );
	friend FString LexToString( EDownloadState DownloadState );
	friend FString LexToString( EProcessingState ProcessingState );
};

#endif //WITH_GRDK
