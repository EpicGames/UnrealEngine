// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKPlatformDLCModule.h"
#include "GDKPlatformDLC.h"
#include "GDKRuntimeModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#if WITH_GRDK
#include "PlatformDLCModule.h"
#include "Templates/SharedPointer.h"
#include "HAL/IConsoleManager.h"
#include "Engine/Engine.h"
#include "Containers/Ticker.h"
#include "Misc/ConfigCacheIni.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
    THIRD_PARTY_INCLUDES_START
    #include <XPackage.h>
    THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif

class FGDKPlatformDLCModule : public IGDKPlatformDLCModule
{
public:

#if WITH_GRDK
	virtual void StartupModule() override
	{
		//fixme: auto-start if chosen?
	}

	virtual void ShutdownModule() override
	{
		SetOnScreenDebugRender(false);
		if (DLCHandler.IsValid())
		{
			DLCHandler->Shutdown();
			DLCHandler.Reset();
		}
	}

	virtual TSharedPtr<IPlatformDLC> GetPlatformDLC() override
	{
		EnsureInitialized();
		return DLCHandler;
	}


	void SetOnScreenDebugRender( bool bEnable )
	{
#if !UE_BUILD_SHIPPING
		static FTSTicker::FDelegateHandle DebugDrawTickHandle;

		bool bIsEnabled = (DebugDrawTickHandle.IsValid());
		if (bIsEnabled == bEnable)
		{
			return;
		}

		if (bEnable)
		{
			EnsureInitialized();
			DebugDrawTickHandle = FTSTicker::GetCoreTicker().AddTicker( TEXT("DLCDebugDraw"), 0.0f, [this](float)
			{
				if (DLCHandler.IsValid())
				{
					GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.0f, FColor::White, DLCHandler->Debug_GetStateDescription(), false);

					int PackageIndex = 0;
					FString PackageDescription;
					FColor PackageColor;
					while (DLCHandler->Debug_GetPackageDescription(PackageIndex++, PackageDescription, PackageColor) )
					{
						GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.0f, PackageColor, PackageDescription, false);
					}
				}
				else
				{
					GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.0f, FColor::White, TEXT("GDK DLC handler has not been created"), false);
				}
				return true;
			});
		}
		else
		{
			FTSTicker::GetCoreTicker().RemoveTicker(DebugDrawTickHandle);
			DebugDrawTickHandle.Reset();
		}
#endif
	}


	void EnsureInitialized()
	{
		// only create GDK DLC handler if the GDK runtime is available
		if (!DLCHandler.IsValid() && IGDKRuntimeModule::Get().IsAvailable())
		{
			DLCHandler = MakeShared<FGDKPlatformDLC>();

			bool bAutoInitialize = true;
			GConfig->GetBool(IPlatformDLC::ConfigSectionName, TEXT("AutoInitialize"), bAutoInitialize, GEngineIni);
			if (bAutoInitialize)
			{
				DLCHandler->InitializeAsync();
			}
		}
	}

	TSharedPtr<FGDKPlatformDLC> DLCHandler;




#else
	// stub when there is no GRDK available
	virtual TSharedPtr<IPlatformDLC> GetPlatformDLC() override
	{
		return nullptr;
	}
#endif //WITH_GRDK
};





#if !UE_BUILD_SHIPPING && WITH_GRDK
static FAutoConsoleCommand CVarGDKDumpDLC(
	TEXT("GDK.DLC.DumpState"),
	TEXT("Dumps the state of the DLC"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FGDKPlatformDLCModule* Module = FModuleManager::LoadModulePtr<FGDKPlatformDLCModule>("GDKPlatformDLC");
		if (Module != nullptr && Module->DLCHandler.IsValid())
		{
			UE_LOGF( LogPlatformDLC, Log, "%ls", *Module->DLCHandler->Debug_GetStateDescription() );

			int PackageIndex = 0;
			FString PackageDescription;
			FColor PackageColor;
			while (Module->DLCHandler->Debug_GetPackageDescription(PackageIndex++, PackageDescription, PackageColor) )
			{
				UE_LOGF( LogPlatformDLC, Log, "%ls", *PackageDescription);
			}
		}
	})
);


static FAutoConsoleCommand CVarGDKDLCShow(
	TEXT("GDK.DLC.DebugShow"),
	TEXT("Show or hide the on-screen debugging"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		FGDKPlatformDLCModule* Module = FModuleManager::LoadModulePtr<FGDKPlatformDLCModule>("GDKPlatformDLC");
		if (Module != nullptr)
		{
			bool bShow = (Args.Num() == 0) || (FCString::Atoi(*Args[0]) != 0);
			Module->SetOnScreenDebugRender(bShow);
		}
	})
);

#endif //!UE_BUILD_SHIPPING && WITH_GRDK



IMPLEMENT_MODULE(FGDKPlatformDLCModule, GDKPlatformDLC);
