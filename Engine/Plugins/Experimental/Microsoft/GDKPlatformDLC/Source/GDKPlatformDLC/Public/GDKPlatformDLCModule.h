// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PlatformDLC.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class GDKPLATFORMDLC_API IGDKPlatformDLCModule : public IPlatformDLCFactoryModule
{
public:
	/**
	 * Singleton-like access to the MS Gaming Runtime module instance
	 * @return Returns IGDKPlatformDLCModule singleton instance, loading the module on demand if needed
	 */
	static inline IGDKPlatformDLCModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IGDKPlatformDLCModule>("GDKPlatformDLC");
	}

	// get the platform DLC for this factory
	virtual TSharedPtr<IPlatformDLC> GetPlatformDLC() = 0;
};

